// Copyright Epic Games, Inc. All Rights Reserved.

#include "IPAddressSteam.h"

namespace FNetworkProtocolTypes
{
	const FLazyName SteamSocketsP2P(TEXT("SteamSocketsP2P"));
	const FLazyName SteamSocketsIP(TEXT("SteamSocketsIP"));
}

TArray<uint8> FInternetAddrSteam::GetRawIp() const
{
	// Due to some API design decisions, we cannot rely on something like the generic bytes member
	// Instead we have to create the raw data ourselves.
	TArray<uint8> RawAddressArray;

	// Same code from IPAddressSteam.
	if (Addr.m_eType == k_ESteamNetworkingIdentityType_SteamID)
	{
		uint64 SteamIDNum = Addr.GetSteamID64();
		const uint8* SteamIdWalk = (uint8*)&SteamIDNum;
		while (RawAddressArray.Num() < sizeof(uint64))
		{
			RawAddressArray.Add(*SteamIdWalk);
			++SteamIdWalk;
		}

		// We want to make sure that these arrays are in big endian format.
#if PLATFORM_LITTLE_ENDIAN
		Algo::Reverse(RawAddressArray);
#endif

		RawAddressArray.EmplaceAt(0, k_ESteamNetworkingIdentityType_SteamID); // Push in the type
	}
	else if (Addr.m_eType == k_ESteamNetworkingIdentityType_IPAddress && Addr.GetIPAddr())
	{
		const SteamNetworkingIPAddr* RawSteamIP = Addr.GetIPAddr();
		RawAddressArray.Add(k_ESteamNetworkingIdentityType_IPAddress);
		for (int32 i = 0; i < UE_ARRAY_COUNT(RawSteamIP->m_ipv6); ++i)
		{
			RawAddressArray.Add(RawSteamIP->m_ipv6[i]);
		}
	}

	return RawAddressArray;
}

void FInternetAddrSteam::SetRawIp(const TArray<uint8>& RawAddr)
{
	// Array is invalid, the type should be the first thing in the array.
	if (RawAddr.Num() <= 1)
	{
		return;
	}

	Addr.Clear();
	uint8 ArrayType = RawAddr[0];

	if (ArrayType == k_ESteamNetworkingIdentityType_SteamID)
	{
		TArray<uint8> WorkingArray = RawAddr;
		WorkingArray.RemoveAt(0); // Remove the type

#if PLATFORM_LITTLE_ENDIAN
		Algo::Reverse(WorkingArray);
#endif
		uint64 NewSteamId = 0;
		for (int32 i = 0; i < WorkingArray.Num(); ++i)
		{
			NewSteamId |= (uint64)WorkingArray[i] << (i * 8);
		}

		Addr.SetSteamID64(NewSteamId);
		ProtocolType = FNetworkProtocolTypes::SteamSocketsP2P;
	}
	else if (ArrayType == k_ESteamNetworkingIdentityType_IPAddress)
	{
		SteamNetworkingIPAddr NewAddr;
		NewAddr.Clear();

		// Skip over the flag bit and copy the array into the structure entirely.
		for (int32 i = 1; i < RawAddr.Num(); ++i)
		{
			NewAddr.m_ipv6[i - 1] = RawAddr[i];
		}

		Addr.SetIPAddr(NewAddr);
		ProtocolType = FNetworkProtocolTypes::SteamSocketsIP;
	}
}

void FInternetAddrSteam::SetIp(const TCHAR* InAddr, bool& bIsValid)
{
	FString InAddrStr(InAddr);
	// See if we're a SteamID.
	if (InAddrStr.StartsWith(STEAM_URL_PREFIX) || InAddrStr.IsNumeric())
	{
		InAddrStr.RemoveFromStart(STEAM_URL_PREFIX);
		FString SteamIPStr, SteamChannelStr;
		// Look for a virtual port
		if (InAddrStr.Split(":", &SteamIPStr, &SteamChannelStr, ESearchCase::CaseSensitive, ESearchDir::FromEnd))
		{
			const uint64 Id = FCString::Atoi64(*SteamIPStr);
			if (Id != 0)
			{
				Addr.SetSteamID64(Id);
				const int32 Channel = FCString::Atoi(*SteamChannelStr);
				if (Channel != 0 || SteamChannelStr == "0")
				{
					P2PVirtualPort = Channel;
					bIsValid = true;
				}
				ProtocolType = FNetworkProtocolTypes::SteamSocketsP2P;
			}
			else
			{
				bIsValid = false;
				return;
			}
		}
		else
		{
			Addr.SetSteamID64(FCString::Atoi64(*InAddrStr));
			ProtocolType = FNetworkProtocolTypes::SteamSocketsP2P;
			bIsValid = true;
			return;
		}
	}
	else if (SteamNetworkingUtils()) // check this because the SteamAPI does not.
	{
		// This is an IP address. Tell Steam to parse it.
		SteamNetworkingIPAddr NewAddress;
		bIsValid = NewAddress.ParseString(TCHAR_TO_ANSI(InAddr));
		ProtocolType = FNetworkProtocolTypes::SteamSocketsIP;
		Addr.SetIPAddr(NewAddress);
	}
}

void FInternetAddrSteam::SetPort(int32 InPort)
{
	if (GetProtocolType() == FNetworkProtocolTypes::SteamSocketsP2P)
	{
		P2PVirtualPort = InPort;
	}
	else
	{
		// There's no way to set a port unless you recreate the entire object.
		// It's a restricted property, with no setters, so we have to jump through some hoops.
		SteamNetworkingIPAddr NewAddressInfo;
		// We don't have to check if the data returned here is valid 
		// as GetProtocolType will have already done it for us.
		const SteamNetworkingIPAddr* InternalAddress = Addr.GetIPAddr();
		FMemory::Memcpy(NewAddressInfo, *InternalAddress);
		NewAddressInfo.m_port = InPort;
		Addr.SetIPAddr(NewAddressInfo);
	}
}

int32 FInternetAddrSteam::GetPort() const
{
	if (GetProtocolType() == FNetworkProtocolTypes::SteamSocketsP2P)
	{
		return P2PVirtualPort;
	}
	else if (const SteamNetworkingIPAddr* InternalAddress = Addr.GetIPAddr())
	{
		// Valve stores this in host byte order as of writing.
		return InternalAddress->m_port;
	}

	return 0;
}

void FInternetAddrSteam::SetAnyAddress()
{
	// This will remove all data such that we can set an any address
	Addr.Clear();
	SteamNetworkingIPAddr NewAddress;
	NewAddress.Clear();
	ProtocolType = FNetworkProtocolTypes::SteamSocketsIP;
	Addr.SetIPAddr(NewAddress);
}

FString FInternetAddrSteam::ToString(bool bAppendPort) const
{
	// Attempt to print out only if we have valid data.
	if (!Addr.IsInvalid())
	{
		if (GetProtocolType() == FNetworkProtocolTypes::SteamSocketsP2P)
		{
			FString BaseResult = FString::Printf(TEXT("%llu"), Addr.GetSteamID64());
			if (bAppendPort)
			{
				BaseResult += FString::Printf(TEXT(":%d"), P2PVirtualPort);
			}
			return BaseResult;
		}

		const SteamNetworkingIPAddr* InternalAddress = Addr.GetIPAddr();
		if (InternalAddress && SteamNetworkingUtils())
		{
			ANSICHAR StrBuffer[SteamNetworkingIPAddr::k_cchMaxString];
			FMemory::Memzero(StrBuffer);
			InternalAddress->ToString(StrBuffer, SteamNetworkingIPAddr::k_cchMaxString, bAppendPort);

			return ANSI_TO_TCHAR(StrBuffer);
		}
	}

	return TEXT("Invalid");
}

uint32 FInternetAddrSteam::GetTypeHash() const
{
	// This object doesn't have any address data at all
	// we should attempt to hash something in it anyways
	if (Addr.IsInvalid())
	{
		// This object is completely blank, we can't do anything at all.
		if (P2PVirtualPort == 0)
		{
			return 0;
		}
		else
		{
			// Otherwise, hash the virtual port.
			return GetTypeHashHelper(P2PVirtualPort);
		}
	}

	if (GetProtocolType() == FNetworkProtocolTypes::SteamSocketsP2P)
	{
		return GetTypeHashHelper(Addr.GetSteamID64());
	}
	return GetTypeHashHelper(ToString(true));
}

FName FInternetAddrSteam::GetProtocolType() const
{
	// By default, an empty Addr structure will have a type of unknown, 
	// which will cause this function to return nullptr.
	if (Addr.GetIPAddr() == nullptr)
	{
		return FNetworkProtocolTypes::SteamSocketsP2P;
	}

	return FNetworkProtocolTypes::SteamSocketsIP;
}
