// Copyright Epic Games, Inc. All Rights Reserved.

#include "SteamSharedModule.h"
#include "Misc/CommandLine.h"
#include "HAL/PlatformProcess.h"
#include "Misc/ConfigCacheIni.h"
#include "SocketSubsystem.h"
#include <steam/isteamgameserver.h>
#include <steam/steamtypes.h>

#ifndef STEAM_SDK_INSTALLED
#error Steam SDK not located! Expected to be found in Engine/Source/ThirdParty/Steamworks/{SteamVersion}
#endif // STEAM_SDK_INSTALLED

#if !UE_BUILD_SHIPPING && !UE_BUILD_SHIPPING_WITH_EDITOR
#include "HAL/PlatformFile.h"
#endif // !UE_BUILD_SHIPPING && !UE_BUILD_SHIPPING_WITH_EDITOR

// Steam API for Initialization
THIRD_PARTY_INCLUDES_START
#include "steam/steam_api.h"
#include "steam/steam_gameserver.h"
THIRD_PARTY_INCLUDES_END

DEFINE_LOG_CATEGORY_STATIC(LogSteamShared, Log, All);

IMPLEMENT_MODULE(FSteamSharedModule, SteamShared);

FString FSteamSharedModule::GetSteamModulePath() const
{
	const FString STEAM_SDK_ROOT_PATH(TEXT("Binaries/ThirdParty/Steamworks"));
#if PLATFORM_WINDOWS

	return FPaths::EngineDir() / STEAM_SDK_ROOT_PATH / STEAM_SDK_VER_PATH / TEXT("Win64/");

#elif PLATFORM_LINUX

	return FPaths::EngineDir() / STEAM_SDK_ROOT_PATH / STEAM_SDK_VER_PATH / TEXT("x86_64-unknown-linux-gnu/");
	
#elif PLATFORM_MAC
	return FPaths::EngineDir() / STEAM_SDK_ROOT_PATH / STEAM_SDK_VER_PATH / TEXT("Mac/");

#elif PLATFORM_ANDROID
	return FPaths::EngineDir() / STEAM_SDK_ROOT_PATH / STEAM_SDK_VER_PATH / TEXT("Android/arm64-v8a/");

#else

	return FString();

#endif	//PLATFORM_WINDOWS
}

bool FSteamSharedModule::CanLoadClientDllsOnServer() const
{
#if PLATFORM_WINDOWS
	if (IsRunningDedicatedServer())
	{
		return true;
	}
#endif

	return false;
}

#if !UE_BUILD_SHIPPING && !UE_BUILD_SHIPPING_WITH_EDITOR
/** Filename containing the appid during development */
#define STEAMAPPIDFILENAME TEXT("steam_appid.txt")

inline FString GetSteamAppIdFilename()
{
#if PLATFORM_ANDROID
	extern FString GExternalFilePath;
	const FString FileBaseDir = GExternalFilePath + TEXT("/../");
#else
	const FString FileBaseDir = FPlatformProcess::BaseDir();
#endif

	return FString::Printf(TEXT("%s%s"), *FileBaseDir, STEAMAPPIDFILENAME);
}

/**
 * Write the steam app id to the steam_appid.txt file before initializing the API
 */
static bool WriteSteamAppIdToDisk()
{
	// Write out the steam_appid.txt file before launching
	int32 SteamDevAppId;

	if (!GConfig->GetInt(TEXT("OnlineSubsystemSteam"), TEXT("SteamDevAppId"), SteamDevAppId, GEngineIni))
	{
		UE_LOG(LogSteamShared, Warning, TEXT("Missing SteamDevAppId key in OnlineSubsystemSteam configuration in .ini file."));
		return false;
	}

	if (SteamDevAppId > 0)
	{
		// Access the physical file writer directly so that we still write next to the executable in CotF builds.
		FString SteamAppIdFilename = GetSteamAppIdFilename();
		TUniquePtr<IFileHandle> Handle(IPlatformFile::GetPlatformPhysical().OpenWrite(*SteamAppIdFilename, false, false));
		if (!Handle)
		{
			UE_LOG(LogSteamShared, Warning, TEXT("Failed to create file: %s"), *SteamAppIdFilename);
			return false;
		}
		else
		{
			TAnsiStringBuilder<16> AppId(InPlace, SteamDevAppId);
			Handle->Write((const uint8*)AppId.GetData(), AppId.Len());
			return true;
		}
	}

	UE_LOG(LogSteamShared, Warning, TEXT("Steam App Id provided (%d) is invalid, must be greater than 0!"), SteamDevAppId);
	return false;
}

/**
 * Deletes the steam app id file from disk
 */
static void DeleteSteamAppIdFromDisk()
{
	const FString SteamAppIdFilename = GetSteamAppIdFilename();
	IPlatformFile& Physical = IPlatformFile::GetPlatformPhysical();
	if (Physical.FileExists(*SteamAppIdFilename))
	{
		Physical.DeleteFile(*SteamAppIdFilename);
	}
}
#endif // !UE_BUILD_SHIPPING && !UE_BUILD_SHIPPING_WITH_EDITOR

void FSteamSharedModule::StartupModule()
{
	// On startup load the modules. Anyone who uses this shared library can guarantee
	// that Steamworks will be started and be ready to go for them.
	LoadSteamModules();
}

void FSteamSharedModule::ShutdownModule()
{
	// Check to see if anyone has a DLL handle still.
	if (AreSteamDllsLoaded())
	{
		// Make sure everyone cleaned up their instances properly.
		if (SteamClientObserver.IsValid()) // If our weakptr is still valid, that means there are cases that didn't clean up somehow.
		{
			// If they have not, warn to them they need to clean up in the future and force the deletion.
			uint32 NumSharedReferences = SteamClientObserver.Pin().GetSharedReferenceCount();
			// Make sure to subtract 1 here as we just created a new sharedptr in order to get the reference count
			// (this sharedptr is out of scope so it does not matter)
			UE_LOG(LogSteamShared, Warning, TEXT("There are still %d additional Steam instances in use. These must be shutdown before unloading the module!"), NumSharedReferences - 1);
		}
		SteamClientObserver.Reset(); // Force the clearing of our weakptr.

		// Do the above but this time with the server observer.
		if (SteamServerObserver.IsValid())
		{
			uint32 NumSharedReferences = SteamServerObserver.Pin().GetSharedReferenceCount();
			UE_LOG(LogSteamShared, Warning, TEXT("There are still %d additional Steam server instances in use. These must be shutdown before unloading the module!"), NumSharedReferences - 1);
		}
		SteamServerObserver.Reset();
	}

	// Here we are no longer loaded, so we need to override any DLL handles still open and unlink the DLLs.
	UnloadSteamModules();

#if !UE_BUILD_SHIPPING && !UE_BUILD_SHIPPING_WITH_EDITOR
	DeleteSteamAppIdFromDisk();
#endif // !UE_BUILD_SHIPPING && !UE_BUILD_SHIPPING_WITH_EDITOR
}

TSharedPtr<class FSteamClientInstanceHandler> FSteamSharedModule::ObtainSteamClientInstanceHandle()
{
	if (SteamClientObserver.IsValid())
	{
		return SteamClientObserver.Pin();
	}
	else
	{
		// Create the original base object, and store our weakptrs.
		TSharedPtr<FSteamClientInstanceHandler> BaseInstance = MakeShareable(new FSteamClientInstanceHandler(this));
		SteamClientObserver = BaseInstance;
		if (BaseInstance->IsInitialized()) // Make sure the SteamAPI was initialized properly.
		{
			// This is safe because we'll end up incrementing the BaseInstance refcounter before we go out of scope.
			// Thus the InstanceHandler will not be deleted before we return.
			return SteamClientObserver.Pin();
		}
		else
		{
			// We don't want to hold any references to failed instances.
			SteamClientObserver = nullptr;
		}
	}

	return nullptr;
}

TSharedPtr<class FSteamServerInstanceHandler> FSteamSharedModule::ObtainSteamServerInstanceHandle()
{
	if (SteamServerObserver.IsValid())
	{
		return SteamServerObserver.Pin();
	}
	else
	{
		// Create the original base object, and store our weakptrs.
		TSharedPtr<FSteamServerInstanceHandler> BaseInstance = MakeShareable(new FSteamServerInstanceHandler(this));
		SteamServerObserver = BaseInstance;
		if (BaseInstance->IsInitialized()) // Make sure the SteamAPI was initialized properly.
		{
			// This is safe because we'll end up incrementing the BaseInstance refcounter before we go out of scope.
			// Thus the InstanceHandler will not be deleted before we return.
			return SteamServerObserver.Pin();
		}
		else
		{
			// We don't want to hold any references to failed instances.
			SteamServerObserver = nullptr;
		}
	}

	return nullptr;
}

bool FSteamSharedModule::AreSteamDllsLoaded() const
{
	bool bLoadedClientDll = true;
	bool bLoadedServerDll = true;

#if LOADING_STEAM_LIBRARIES_DYNAMICALLY
	bLoadedClientDll = (SteamDLLHandle != nullptr) ? true : false;
#endif // LOADING_STEAM_LIBRARIES_DYNAMICALLY
#if LOADING_STEAM_SERVER_LIBRARY_DYNAMICALLY
	bLoadedServerDll = IsRunningDedicatedServer() ? ((SteamServerDLLHandle != nullptr || !bForceLoadSteamClientDll) ? true : false) : true;
#endif // LOADING_STEAM_SERVER_LIBRARY_DYNAMICALLY

	return bLoadedClientDll && bLoadedServerDll;
}

void FSteamSharedModule::LoadSteamModules()
{
	if (AreSteamDllsLoaded())
	{
		return;
	}

	UE_LOG(LogSteamShared, Display, TEXT("Loading Steam SDK %s"), STEAM_SDK_VER);

#if PLATFORM_WINDOWS

	FString RootSteamPath = GetSteamModulePath();
	FPlatformProcess::PushDllDirectory(*RootSteamPath);
	SteamDLLHandle = FPlatformProcess::GetDllHandle(*(RootSteamPath + "steam_api64.dll"));
	if (IsRunningDedicatedServer() && FCommandLine::IsInitialized() && FParse::Param(FCommandLine::Get(), TEXT("force_steamclient_link")))
	{
		FString SteamClientDLL("steamclient64.dll"),
			SteamTierDLL("tier0_s64.dll"),
			SteamVSTDDLL("vstdlib_s64.dll");

		UE_LOG(LogSteamShared, Log, TEXT("Attempting to force linking the steam client dlls."));
		bForceLoadSteamClientDll = true;
		SteamServerDLLHandle = FPlatformProcess::GetDllHandle(*(RootSteamPath + SteamClientDLL));
		if(!SteamServerDLLHandle)
		{
			UE_LOG(LogSteamShared, Error, TEXT("Could not find the %s, %s and %s DLLs, make sure they are all located at %s! These dlls can be located in your Steam install directory."),
				*SteamClientDLL, *SteamTierDLL, *SteamVSTDDLL, *RootSteamPath);
		}
	}
	FPlatformProcess::PopDllDirectory(*RootSteamPath);
#elif PLATFORM_MAC || (PLATFORM_LINUX && LOADING_STEAM_LIBRARIES_DYNAMICALLY) || (PLATFORM_ANDROID && LOADING_STEAM_LIBRARIES_DYNAMICALLY)

#if PLATFORM_MAC
	FString SteamModuleFileName("libsteam_api.dylib");
#else
	UE_LOGF(LogSteamShared, Warning, "Loading libsteam_api.so");
	FString SteamModuleFileName("libsteam_api.so");
#endif // PLATFORM_MAC

	SteamDLLHandle = FPlatformProcess::GetDllHandle(*SteamModuleFileName);
	if (SteamDLLHandle == nullptr)
	{
		// try bundled one
#if PLATFORM_MAC
		if (FParse::Param(FCommandLine::Get(), TEXT("dllerrors")))
#endif // PLATFORM_MAC
		{
			UE_LOG(LogSteamShared, Warning, TEXT("Could not find system one, loading bundled %s."), *SteamModuleFileName);
		}
		FString RootSteamPath = GetSteamModulePath();
		SteamDLLHandle = FPlatformProcess::GetDllHandle(*(RootSteamPath + SteamModuleFileName));
	}

	if (SteamDLLHandle)
	{
		UE_LOG(LogSteamShared, Display, TEXT("Loaded %s at %p"), *SteamModuleFileName, SteamDLLHandle);
	}
	else
	{
		UE_LOG(LogSteamShared, Warning, TEXT("Unable to load %s, Steam functionality will not work"), *SteamModuleFileName);
		return;
	}


#elif PLATFORM_LINUX
	UE_LOG(LogSteamShared, Log, TEXT("libsteam_api.so is linked explicitly and should be already loaded."));
	return;
#endif // PLATFORM_WINDOWS
	UE_LOG(LogSteamShared, Log, TEXT("Steam SDK Loaded!"));
}

void FSteamSharedModule::UnloadSteamModules()
{
	// Only free the handles if no one is using them anymore.
	// There's no need to check AreSteamDllsLoaded as this is done individually below.
	if (!SteamClientObserver.IsValid() && !SteamServerObserver.IsValid())
	{
#if LOADING_STEAM_LIBRARIES_DYNAMICALLY
		UE_LOG(LogSteamShared, Log, TEXT("Freeing the Steam Loaded Modules..."));

		if (SteamDLLHandle != nullptr)
		{
			FPlatformProcess::FreeDllHandle(SteamDLLHandle);
			SteamDLLHandle = nullptr;
		}

		if (SteamServerDLLHandle != nullptr)
		{
			FPlatformProcess::FreeDllHandle(SteamServerDLLHandle);
			SteamServerDLLHandle = nullptr;
		}
#endif	//LOADING_STEAM_LIBRARIES_DYNAMICALLY
	}
}

FSteamInstanceHandlerBase::FSteamInstanceHandlerBase() : 
	bInitialized(false)
{
	// Grab the gameport for game communications.
	if (FParse::Value(FCommandLine::Get(), TEXT("Port="), GamePort) == false)
	{
		GConfig->GetInt(TEXT("URL"), TEXT("Port"), GamePort, GEngineIni);
	}
}

FSteamClientInstanceHandler::FSteamClientInstanceHandler(FSteamSharedModule* SteamInitializer) :
	FSteamInstanceHandlerBase()
{
#if !STEAM_UNSUPPORTED_ANDROID_PLATFORMS
	// A module must be loaded in order for us to initialize the Steam API.
	if (SteamInitializer != nullptr && SteamInitializer->AreSteamDllsLoaded())
	{
#if !UE_BUILD_SHIPPING && !UE_BUILD_SHIPPING_WITH_EDITOR
		if (!WriteSteamAppIdToDisk())
		{
			UE_LOG(LogSteamShared, Warning, TEXT("Failed to create steam_appid.txt. If the file doesn't exist, SteamAPI_Init will fail."));
		}
#endif // !UE_BUILD_SHIPPING && !UE_BUILD_SHIPPING_WITH_EDITOR 
		SteamErrMsg InitServerErrorMsg;
		if (SteamAPI_InitEx(&InitServerErrorMsg) == k_ESteamAPIInitResult_OK)
		{
			UE_LOG(LogSteamShared, Verbose, TEXT("SteamAPI initialized"));
			bInitialized = true;
			return;
		}

		// The conditions mentioned in this print can be found at https://partner.steamgames.com/doc/sdk/api#initialization_and_shutdown
		UE_LOG(LogSteamShared, Warning, TEXT("SteamAPI failed to initialize: %s"), StringCast<TCHAR>(InitServerErrorMsg).Get());
		return;
	}
	UE_LOG(LogSteamShared, Warning, TEXT("SteamAPI failed to initialize as the Dlls are not loaded."));
#else
	// The Steam SDK is unsupported on these platforms, but for binary builds of the engine, we may have to bail here to avoid initializing
	UE_LOG(LogSteamShared, Warning, TEXT("SteamAPI failed to initialize.  On Android, only Arm64 platforms are supported."));
	bInitialized = false;
	return;
#endif //!STEAM_UNSUPPORTED_ANDROID_PLATFORMS
}

void FSteamClientInstanceHandler::InternalShutdown()
{
	UE_LOG(LogSteamShared, Log, TEXT("Unloading the Steam API..."));
	SteamAPI_Shutdown();
}


FSteamServerInstanceHandler::FSteamServerInstanceHandler(FSteamSharedModule* SteamInitializer) :
	FSteamInstanceHandlerBase()
{
#if !STEAM_UNSUPPORTED_ANDROID_PLATFORMS
	if (SteamInitializer == nullptr || !SteamInitializer->AreSteamDllsLoaded())
	{
		UE_LOG(LogSteamShared, Warning, TEXT("Steam Server API failed to initialize as the Dlls are not loaded."));
		return;
	}

	// Get the multihome address. If there isn't one, this server will be set to listen on the any address.
	uint32 LocalServerIP = 0;
	FString MultiHome;
	if (FParse::Value(FCommandLine::Get(), TEXT("MULTIHOME="), MultiHome) && !MultiHome.IsEmpty())
	{
		TSharedPtr<FInternetAddr> MultiHomeIP = ISocketSubsystem::Get(PLATFORM_SOCKETSUBSYSTEM)->GetAddressFromString(MultiHome);
		if (MultiHomeIP.IsValid())
		{
			MultiHomeIP->GetIp(LocalServerIP);
		}
	}

	// Allow the command line to override the default query port for server list queries
	if (FParse::Value(FCommandLine::Get(), TEXT("QueryPort="), QueryPort) == false)
	{
		if (!GConfig->GetInt(TEXT("OnlineSubsystemSteam"), TEXT("GameServerQueryPort"), QueryPort, GEngineIni))
		{
			QueryPort = 27015;
		}
	}

	// Set VAC
	bool bVACEnabled = false;
	GConfig->GetBool(TEXT("OnlineSubsystemSteam"), TEXT("bVACEnabled"), bVACEnabled, GEngineIni);

	// Set GameVersions
	FString GameVersion;
	GConfig->GetString(TEXT("OnlineSubsystemSteam"), TEXT("GameVersion"), GameVersion, GEngineIni);
	if (GameVersion.Len() == 0)
	{
		UE_LOG(LogSteamShared, Warning, TEXT("[OnlineSubsystemSteam].GameVersion is not set. Server advertising will fail"));
	}

	UE_LOG(LogSteamShared, Verbose, TEXT("Initializing Steam Game Server IP: 0x%08X Port: %d QueryPort: %d"), LocalServerIP, GamePort, QueryPort);

#if !UE_BUILD_SHIPPING && !UE_BUILD_SHIPPING_WITH_EDITOR
	if (!WriteSteamAppIdToDisk())
	{
		UE_LOG(LogSteamShared, Warning, TEXT("Failed to create steam_appid.txt. If the file doesn't exist, SteamAPI_Init will fail."));
	}
#endif // !UE_BUILD_SHIPPING && !UE_BUILD_SHIPPING_WITH_EDITOR 
	
	SteamErrMsg InitServerErrorMsg;
	ESteamAPIInitResult InitStatus = SteamGameServer_InitEx(LocalServerIP, GamePort, QueryPort,
		(bVACEnabled ? eServerModeAuthenticationAndSecure : eServerModeAuthentication),
		TCHAR_TO_UTF8(*GameVersion), &InitServerErrorMsg);
	if (InitStatus == k_ESteamAPIInitResult_OK)
	{
		UE_LOG(LogSteamShared, Verbose, TEXT("Steam Dedicated Server API initialized."));
		bInitialized = true;
	}
	else
	{
		UE_LOG(LogSteamShared, Warning, TEXT("Steam Dedicated Server API failed to initialize: %s"), StringCast<TCHAR>(InitServerErrorMsg).Get());
	}
#else
	// The Steam SDK is unsupported on these platforms, but for binary builds of the engine, we may have to bail here to avoid initializing
	UE_LOG(LogSteamShared, Warning, TEXT("SteamAPI failed to initialize.  On Android, only Arm64 platforms are supported."));
	bInitialized = false;
	return;
#endif //!STEAM_UNSUPPORTED_ANDROID_PLATFORMS
}

void FSteamServerInstanceHandler::InternalShutdown()
{
	// Log off of the steam coordinator servers, removes this server immediately from the backend.
	if (SteamGameServer() && SteamGameServer()->BLoggedOn())
	{
		SteamGameServer()->LogOff();
	}

	// By getting here, we must be the last instance of the object, thus we should be deleted.
	// If no one is using the API, we can shut it down.
	UE_LOG(LogSteamShared, Log, TEXT("Unloading the Steam server API..."));
	SteamGameServer_Shutdown();
}

bool FSteamInstanceHandlerBase::CanCleanUp() const
{
	return bInitialized && FSteamSharedModule::IsAvailable() && FSteamSharedModule::Get().AreSteamDllsLoaded();
}

void FSteamInstanceHandlerBase::Destroy()
{
	if (CanCleanUp())
	{
		InternalShutdown();
	}
}

#if STEAM_UNSUPPORTED_ANDROID_PLATFORMS
// These stubs make it safe to work with unsupported Android platforms by stubbing out these API functions that are called, so that they
// don't try to link against an actual library where there is none.

S_API bool S_CALLTYPE SteamAPI_RestartAppIfNecessary(uint32 unOwnAppID)
{
	return false;
}

S_API void* S_CALLTYPE SteamInternal_ContextInit(void* pContextInitData)
{
	return nullptr;
}

S_API void S_CALLTYPE SteamAPI_RegisterCallback(class CCallbackBase* pCallback, int iCallback)
{
}

S_API void S_CALLTYPE SteamAPI_RunCallbacks()
{
}

S_API void S_CALLTYPE SteamGameServer_RunCallbacks()
{
}

S_API void S_CALLTYPE SteamAPI_UnregisterCallback(class CCallbackBase* pCallback)
{
}

S_API HSteamUser S_CALLTYPE SteamAPI_GetHSteamUser()
{
	return (HSteamUser)0;
}

S_API void* S_CALLTYPE SteamInternal_FindOrCreateUserInterface(HSteamUser hSteamUser, const char* pszVersion)
{
	return nullptr;
}

S_API HSteamUser S_CALLTYPE SteamGameServer_GetHSteamUser()
{
	return (HSteamUser)0;
}

S_API void* S_CALLTYPE SteamInternal_FindOrCreateGameServerInterface(HSteamUser hSteamUser, const char* pszVersion)
{
	return nullptr;
}

S_API void S_CALLTYPE SteamAPI_Shutdown()
{
}

S_API void SteamGameServer_Shutdown()
{
}

#endif //STEAM_UNSUPPORTED_ANDROID_PLATFORMS