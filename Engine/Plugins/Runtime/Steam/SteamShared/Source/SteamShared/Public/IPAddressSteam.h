// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IPAddress.h"
#include "SteamOnlineDefines.h"

THIRD_PARTY_INCLUDES_START
// IWYU pragma: begin_exports
#include "steam/steam_api.h"
#include "steam/steam_gameserver.h"
#include "steam/isteamnetworkingsockets.h"
#include "steam/steamnetworkingtypes.h"

#define UE_API STEAMSHARED_API
// IWYU pragma: end_exports
THIRD_PARTY_INCLUDES_END

#define STEAM_URL_PREFIX TEXT("steam.")

class FInternetAddrSteam : public FInternetAddr
{
private:
	SteamNetworkingIdentity Addr;
	int32 P2PVirtualPort;
	FName ProtocolType;

public:
	FInternetAddrSteam(const FName RequestedProtocol = NAME_None) :
		P2PVirtualPort(0),
		ProtocolType(RequestedProtocol)
	{
		Addr.Clear();
	}

	FInternetAddrSteam(const SteamNetworkingIdentity& NewAddress) :
		Addr(NewAddress),
		P2PVirtualPort(0),
		ProtocolType(NewAddress.GetIPAddr() == nullptr ? FNetworkProtocolTypes::SteamSocketsP2P : FNetworkProtocolTypes::SteamSocketsIP)
	{
	}

	FInternetAddrSteam(const SteamNetworkingIPAddr& IPAddr) :
		P2PVirtualPort(0),
		ProtocolType(FNetworkProtocolTypes::SteamSocketsIP)
	{
		Addr.SetIPAddr(IPAddr);
	}

	explicit FInternetAddrSteam(uint64& SteamID) :
		P2PVirtualPort(0),
		ProtocolType(FNetworkProtocolTypes::SteamSocketsP2P)
	{
		Addr.SetSteamID64(SteamID);
	}

	explicit FInternetAddrSteam(const CSteamID& SteamID) :
		P2PVirtualPort(0),
		ProtocolType(FNetworkProtocolTypes::SteamSocketsP2P)
	{
		Addr.SetSteamID(SteamID);
	}

	UE_API virtual TArray<uint8> GetRawIp() const override;

	UE_API virtual void SetRawIp(const TArray<uint8>& RawAddr) override;

	virtual void SetIp(uint32 InAddr) override
	{
		/** Not used */
	}

	/**
	* Attempts to return the CSteamID of the target.
	* Will return an invalid CSteamID if using IP address
	*/ 
	CSteamID GetSteamID() const
	{
		return Addr.GetSteamID();
	}
	uint64 GetSteamID64() const
	{
		return Addr.GetSteamID().ConvertToUint64();
	}

	/**
	 * Shortcut for SetIp because SetIp is expecting the array format from GetIp which has some additional information
	 */
	void SetSteamID(CSteamID NewSteamID)
	{
		ProtocolType = FNetworkProtocolTypes::SteamSocketsP2P;
		Addr.SetSteamID(NewSteamID);
	}


	/**
	* Sets the ip address from a string ("A.B.C.D") or a steam id "steam.STEAMID" or "STEAMID:PORT"
	*
	* @param InAddr the string containing the new ip address to use
	*/
	UE_API virtual void SetIp(const TCHAR* InAddr, bool& bIsValid) override;

	/**
	* Copies the network byte order ip address to a host byte order dword
	*
	* @param OutAddr the out param receiving the ip address
	*/
	virtual void GetIp(uint32& OutAddr) const override
	{
		/** Not used */
	}

	/**
	* Sets the port number from a host byte order int
	*
	* @param InPort the new port to use (must convert to network byte order)
	*/
	UE_API virtual void SetPort(int32 InPort) override;

	/**
	* Returns the port number from this address in host byte order
	*/
	UE_API virtual int32 GetPort() const override;

	/**
	 * Set Platform specific port data
	 */
	virtual void SetPlatformPort(int32 InPort) override
	{
		P2PVirtualPort = (int16)InPort;
	}

	/**
	 * Get platform specific port data.
	 */
	virtual int32 GetPlatformPort() const override
	{
		return (int32)P2PVirtualPort;
	}

	/** Sets the address to be any address */
	UE_API virtual void SetAnyAddress() override;

	/** Sets the address to broadcast */
	virtual void SetBroadcastAddress() override
	{
		/** Not used */
	}

	/** Sets the address to loopback */
	virtual void SetLoopbackAddress() override
	{
		Addr.SetLocalHost();
	}

	/**
	* Converts this internet ip address to string form
	*
	* @param bAppendPort whether to append the port information or not
	*/
	UE_API virtual FString ToString(bool bAppendPort) const override;

	/**
	* Compares two internet ip addresses for equality
	*
	* @param Other the address to compare against
	*/
	virtual inline bool operator==(const FInternetAddr& Other) const override
	{
		FInternetAddrSteam& SteamOther = (FInternetAddrSteam&)Other;
		return Addr == SteamOther.Addr;
	}

	/**
	* Compares two internet ip addresses for equality
	*
	* @param Other the address to compare against
	*/
	inline bool operator==(const FInternetAddrSteam& Other) const
	{
		return Addr == Other.Addr;
	}

	UE_API virtual uint32 GetTypeHash() const override;

	UE_API virtual FName GetProtocolType() const override;

	virtual bool IsValid() const override
	{
		return !(Addr.IsInvalid());
	}

	operator const SteamNetworkingIPAddr() const
	{
		const SteamNetworkingIPAddr* IPAddr = Addr.GetIPAddr();
		if (IPAddr == nullptr)
		{
			SteamNetworkingIPAddr EmptyAddr;
			EmptyAddr.Clear();
			return EmptyAddr;
		}
		return *IPAddr;
	}

	operator const SteamNetworkingIdentity() const
	{
		return Addr;
	}

	virtual TSharedRef<FInternetAddr> Clone() const override
	{
		TSharedRef<FInternetAddrSteam> NewAddress = MakeShareable(new FInternetAddrSteam(ProtocolType));
		NewAddress->Addr = Addr;
		NewAddress->P2PVirtualPort = P2PVirtualPort;
		return NewAddress;
	}
};

#undef UE_API
