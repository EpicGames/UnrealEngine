// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Modules/ModuleInterface.h"
#include "Templates/SharedPointer.h"

class ISteamSharedModule : public IModuleInterface
{
public:
	/** Retrieve steam client handle if steam is enabled, otherwise return a nullptr */
	virtual TSharedPtr<class FSteamClientInstanceHandler> ObtainSteamClientInstanceHandle() = 0;
};
