// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once
#include "UObject/NameTypes.h"

namespace FNetworkProtocolTypes
{
	STEAMSHARED_API extern const FLazyName SteamSocketsP2P;
	STEAMSHARED_API extern const FLazyName SteamSocketsIP;
}