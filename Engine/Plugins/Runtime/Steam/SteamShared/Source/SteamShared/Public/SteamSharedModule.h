// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "ISteamSharedModule.h"
#include "Modules/ModuleInterface.h"
#include "Modules/ModuleManager.h"
#include "SteamSharedPackage.h"

#define UE_API STEAMSHARED_API

#define LOADING_STEAM_CLIENT_LIBRARY_DYNAMICALLY		(PLATFORM_WINDOWS || PLATFORM_MAC || PLATFORM_ANDROID || (PLATFORM_LINUX && !IS_MONOLITHIC))
#define LOADING_STEAM_SERVER_LIBRARY_DYNAMICALLY		(PLATFORM_WINDOWS || (PLATFORM_LINUX && !IS_MONOLITHIC) || PLATFORM_MAC)
#define LOADING_STEAM_LIBRARIES_DYNAMICALLY				(LOADING_STEAM_CLIENT_LIBRARY_DYNAMICALLY || LOADING_STEAM_SERVER_LIBRARY_DYNAMICALLY)

#define STEAM_UNSUPPORTED_ANDROID_PLATFORMS				(PLATFORM_ANDROID_X64 || PLATFORM_ANDROID_X86)

class FSteamSharedModule : public ISteamSharedModule
{
public:

	FSteamSharedModule() : 
		SteamDLLHandle(nullptr),
		SteamServerDLLHandle(nullptr),
		bForceLoadSteamClientDll(false),
		SteamClientObserver(nullptr),
		SteamServerObserver(nullptr)
	{
	}

	virtual ~FSteamSharedModule() = default;

	// IModuleInterface
	UE_API virtual void StartupModule() override;
	UE_API virtual void ShutdownModule() override;

	// Due to the loading of the DLLs and how the Steamworks API is initialized, we cannot support dynamic reloading.
	virtual bool SupportsDynamicReloading() override
	{
		return false;
	}

	/**
	 * Initializes Steam Client API and provides a handler that will keep the API valid for the lifetime of the
	 * the object. Several Handlers can be active at once. 
	 *
	 * @return A handler to the Steam Client API, use IsValid to check if the handle is initialized.
	 */
	UE_API virtual TSharedPtr<class FSteamClientInstanceHandler> ObtainSteamClientInstanceHandle() override;

	/**
	 * Initializes Steam Server API and provides a handler that will keep the API valid for the lifetime of the
	 * the object. Several Handlers can be active at once.
	 *
	 * @return A handler to the Steam Server API, use IsValid to check if the handle is initialized.
	 */
	UE_API TSharedPtr<class FSteamServerInstanceHandler> ObtainSteamServerInstanceHandle();
	
	/**
	 * Are the Steamworks Dlls loaded
	 *
	 * @return if the steam dlls are currently loaded (if we are loading them dynamically, statically linked are always true)
	 */
	UE_API bool AreSteamDllsLoaded() const;
	
	/**
	 * The path to where the Steam binaries are stored, for use in debugging.
	 *
	 * @return The directory path of the location of the steam dlls
	 */
	UE_API FString GetSteamModulePath() const;

	/**
	 * If the module will be loading the client dlls for the dedicated server instance.
	 * Really only useful on Windows.
	 *
	 * @return If this application is currently loading client dlls on the server
	 */
	bool IsLoadingServerClientDlls() const { return bForceLoadSteamClientDll; }

	/**
	 * Checks if we can load client dlls on dedicated server instances.
	 * 
	 * @return On dedicated servers on windows, this returns true, for other platforms this returns false (as feature is unnecessary)
	 */
	UE_API bool CanLoadClientDllsOnServer() const;

	/**
	 * Singleton-like access to this module's interface.  This is just for convenience!
	 * Beware of calling this during the shutdown phase, though.  Your module might have been unloaded already.
	 *
	 * @return Returns singleton instance, loading the module on demand if needed
	 */
	static inline class FSteamSharedModule& Get()
	{
		return FModuleManager::LoadModuleChecked<class FSteamSharedModule>("SteamShared");
	}

	/**
	 * Checks to see if this module is loaded and ready.  It is only valid to call Get() if IsAvailable() returns true.
	 *
	 * @return True if the module is loaded and ready to use
	 */
	static inline bool IsAvailable()
	{
		return FModuleManager::Get().IsModuleLoaded("SteamShared");
	}

private:

	/** Handle to the STEAM API dll */
	void* SteamDLLHandle;

	/** Handle to the STEAM dedicated server support dlls */
	void* SteamServerDLLHandle;

	/** If we force loaded the steamclient dlls due to launch flags */
	bool bForceLoadSteamClientDll;

	/** Object that holds the refcounted pointer that's given out */
	TWeakPtr<class FSteamClientInstanceHandler> SteamClientObserver;
	TWeakPtr<class FSteamServerInstanceHandler> SteamServerObserver;

	/** Load the required modules for Steam */
	UE_API void LoadSteamModules();

	/** Unload the required modules for Steam */
	UE_API void UnloadSteamModules();
};

/** Base instance handler class for the Steam shared classes, this allows for less code redundancy between the shared modules. */
class FSteamInstanceHandlerBase
{
public:
	virtual ~FSteamInstanceHandlerBase() = default;
	virtual bool IsInitialized() const { return bInitialized; }
	int32 GetGamePort() const { return GamePort; }

protected:
	UE_API FSteamInstanceHandlerBase();

	bool bInitialized;
	int32 GamePort;

	UE_API virtual bool CanCleanUp() const;
	UE_API virtual void Destroy();
	virtual void InternalShutdown() = 0;
};

/** A simple instance handler that creates and uninitializes the client SteamAPI automatically. */
class FSteamClientInstanceHandler final : public FSteamInstanceHandlerBase
{
public:
	virtual ~FSteamClientInstanceHandler() { Destroy(); }

PACKAGE_SCOPE:
	/** Initializes the Steamworks client API on call */	
	FSteamClientInstanceHandler(FSteamSharedModule* SteamInitializer);

protected:
	UE_API virtual void InternalShutdown() override;

private:
	FSteamClientInstanceHandler() : 
		FSteamInstanceHandlerBase()
	{
	}
};

/** A simple instance handler that creates and uninitializes the server SteamAPI automatically. */
class FSteamServerInstanceHandler final : public FSteamInstanceHandlerBase
{
public:
	virtual ~FSteamServerInstanceHandler() { Destroy(); }

	int32 GetQueryPort() const { return QueryPort; }

PACKAGE_SCOPE:
	/** Initializes the Steamworks server API on call */
	FSteamServerInstanceHandler(FSteamSharedModule* SteamInitializer);

protected:
	int32 QueryPort;
	UE_API virtual void InternalShutdown() override;

private:
	FSteamServerInstanceHandler() : 
		FSteamInstanceHandlerBase(),
		QueryPort(-1)
	{
	}
};

#undef UE_API