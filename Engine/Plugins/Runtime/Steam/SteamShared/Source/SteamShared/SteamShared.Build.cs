// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;
using System.IO;

public class SteamShared : ModuleRules
{
	public SteamShared(ReadOnlyTargetRules Target) : base(Target)
	{
		PCHUsage = ModuleRules.PCHUsageMode.UseExplicitOrSharedPCHs;
		PrivateDefinitions.Add("STEAMSHARED_PACKAGE=1");

		// Check if the Steam SDK exists.
		if (Directory.Exists(Target.UEThirdPartySourceDirectory + "Steamworks/"))
        {
            PublicDefinitions.Add("STEAM_SDK_INSTALLED");
        }
	
		PrivateDependencyModuleNames.AddRange(
			new string[] {
				"Core", 
				"Sockets"
            }
		);

		AddEngineThirdPartyPrivateStaticDependencies(Target, "Steamworks");
	}
}