{
	"FileVersion": 3,
	"Version": 1,
	"VersionName": "1.0",
	"FriendlyName": "Steam Shared Module",
	"Description": "Shared module loader for the Steam API",
	"Category": "Online Platform",
	"CreatedBy": "Epic Games, Inc.",
	"CreatedByURL": "https://epicgames.com",
	"EnabledByDefault": false,
	"CanContainContent": false,
	"Modules": [
		{
			"Name": "SteamShared",
			"Type": "Runtime",
			"LoadingPhase": "PreDefault",
			"PlatformAllowList": [
				"Win64",
				"Mac",
				"Linux",
				"Android"
			],
			"PlatformArchitectureDenyList": [
				"Win64:arm64"
			]
		}
	],
	"DocsURL": "",
	"MarketplaceURL": "",
	"SupportURL": "",
	"IsBetaVersion": false,
	"IsExperimentalVersion": false,
	"Installed": false
}